{
	"FileVersion" : 3,
	"Version" : 2,
	"VersionName" : "2.0",
	"FriendlyName" : "CLion Integration",
	"Description" : "Allows access to source code in CLion.",
	"Category" : "Programming",
	"CreatedBy" : "dotBunny, Inc.",
	"CreatedByURL" : "https://dotbunny.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "CLionSourceCodeAccess",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Mac",
				"Linux",
				"Win64"
			]
		}
	]
}
