// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class CLionSourceCodeAccess : ModuleRules
	{
        public CLionSourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
		    PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "Core",
                    "SourceCodeAccess",
                    "DesktopPlatform",
                    "Json"
                }
            );

            if (Target.bBuildEditor)
            {
                PrivateDependencyModuleNames.Add("HotReload");
            }

			ShortName = "CLionSCA";
		}
	}
}