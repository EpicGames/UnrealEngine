// Copyright Epic Games, Inc. All Rights Reserved.

#include "CLionSourceCodeAccessModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "HAL/LowLevelMemTracker.h"

#define LOCTEXT_NAMESPACE "CLionSourceCodeAccessor"

LLM_DEFINE_TAG(CLionSourceCodeAccess);

IMPLEMENT_MODULE(FCLionSourceCodeAccessModule, CLionSourceCodeAccess);

void FCLionSourceCodeAccessModule::ShutdownModule()
{
	// Unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature(TEXT("SourceCodeAccessor"), &CLionSourceCodeAccessor);
}

void FCLionSourceCodeAccessModule::StartupModule()
{
	LLM_SCOPE_BYTAG(CLionSourceCodeAccess);

	// Quick forced check of availability before anyone touches the module
	CLionSourceCodeAccessor.RefreshAvailability();

	// Bind our source control provider to the editor
	IModularFeatures::Get().RegisterModularFeature(TEXT("SourceCodeAccessor"), &CLionSourceCodeAccessor);
}

bool FCLionSourceCodeAccessModule::SupportsDynamicReloading()
{
	return true;
}

FCLionSourceCodeAccessor& FCLionSourceCodeAccessModule::GetAccessor()
{
	return CLionSourceCodeAccessor;
}

#undef LOCTEXT_NAMESPACE
