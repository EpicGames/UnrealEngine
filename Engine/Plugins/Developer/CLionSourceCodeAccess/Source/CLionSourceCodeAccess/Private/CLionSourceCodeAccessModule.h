// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "CLionSourceCodeAccessor.h"

class FCLionSourceCodeAccessModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;

	virtual void ShutdownModule() override;

	virtual bool SupportsDynamicReloading() override;

	FCLionSourceCodeAccessor& GetAccessor();
private:

	FCLionSourceCodeAccessor CLionSourceCodeAccessor;

};
