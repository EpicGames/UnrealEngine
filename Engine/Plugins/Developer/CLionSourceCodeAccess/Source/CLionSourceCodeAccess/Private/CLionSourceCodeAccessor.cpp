// Copyright Epic Games, Inc. All Rights Reserved.

#include "CLionSourceCodeAccessor.h"
#include "HAL/PlatformProcess.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

#if PLATFORM_WINDOWS
#include "Internationalization/Regex.h"
#include "Serialization/JsonSerializer.h"
#include "Trace/Trace.inl"
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#define LOCTEXT_NAMESPACE "CLionSourceCodeAccessor"

DEFINE_LOG_CATEGORY_STATIC(LogCLionAccessor, Log, All);

void FCLionSourceCodeAccessor::RefreshAvailability()
{
	// Find our program
	ExecutablePath = FindExecutablePath();

	// If we have an executable path, we certainly have it installed!
	if (!ExecutablePath.IsEmpty())
	{
		bHasCLionInstalled = true;
	}
	else
	{
		bHasCLionInstalled = false;
	}
}

bool FCLionSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	// @todo.clion Manually add to folders? Or just regenerate
	return false;
}

bool FCLionSourceCodeAccessor::CanAccessSourceCode() const
{
	return bHasCLionInstalled;
}

bool FCLionSourceCodeAccessor::DoesSolutionExist() const
{
	FString Path = FPaths::Combine(*FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("CMakeLists.txt"));
	if (FPaths::FileExists(Path))
	{
		return true;
	}
	// Check for this project being included as part of the engine one
	Path = FPaths::Combine(*FPaths::ConvertRelativePathToFull(FPaths::RootDir()), TEXT("CMakeLists.txt"));
	if (FPaths::FileExists(Path))
	{
		return true;
	}
	return false;
}

FText FCLionSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("CLionDisplayDesc", "Open source code files in CLion");
}

FName FCLionSourceCodeAccessor::GetFName() const
{
	return FName("CLionSourceCodeAccessor");
}

FText FCLionSourceCodeAccessor::GetNameText() const
{
	return LOCTEXT("CLionDisplayName", "CLion");
}

bool FCLionSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	if (!bHasCLionInstalled)
	{
		return false;
	}

	const FString Path = FString::Printf(TEXT("\"%s\" --line %d \"%s\""), *FPaths::ConvertRelativePathToFull(*FPaths::ProjectDir()), LineNumber, *FullPath);

	FProcHandle Proc = FPlatformProcess::CreateProc(*ExecutablePath, *Path, true, true, false, nullptr, 0, nullptr, nullptr);
	if (!Proc.IsValid())
	{
		UE_LOGF(LogCLionAccessor, Warning, "Opening file (%ls) at a specific line failed.", *Path);
		FPlatformProcess::CloseProc(Proc);
		return false;
	}

	return true;
}

bool FCLionSourceCodeAccessor::OpenSolution()
{
	if (!bHasCLionInstalled)
	{
		return false;
	}

	const FString Path = FString::Printf(TEXT("\"%s\""), *FPaths::ConvertRelativePathToFull(*FPaths::ProjectDir()));

	FPlatformProcess::CreateProc(*ExecutablePath, *Path, true, true, false, nullptr, 0, nullptr, nullptr);

	return true;
}

bool FCLionSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	if (!bHasCLionInstalled)
	{
		return false;
	}

	FString CorrectSolutionPath = InSolutionPath;
	if (InSolutionPath.EndsWith(TEXT("UE5")))
	{
		CorrectSolutionPath.LeftInline(CorrectSolutionPath.Len() - 3);
	}
	// UE5 passes the project folder and name, so strip the name off
	int32 LastPathIndex = CorrectSolutionPath.Find(TEXT("/"), ESearchCase::CaseSensitive, ESearchDir::FromEnd);
	if (LastPathIndex > -1)
	{
		CorrectSolutionPath.LeftInline(LastPathIndex + 1);
	}
	// Make sure the path is wrapped in "" properly
	CorrectSolutionPath = FString::Printf(TEXT("\"%sCMakeLists.txt\""), *CorrectSolutionPath);

	return FPlatformProcess::CreateProc(*ExecutablePath, *CorrectSolutionPath, true, true, false, nullptr, 0, nullptr, nullptr).IsValid();
}

bool FCLionSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	if (!bHasCLionInstalled)
	{
		return false;
	}

	FString SourceFilesList = "";

	// Build our paths based on what unreal sends to be opened
	for (const auto& SourcePath : AbsoluteSourcePaths)
	{
		SourceFilesList = FString::Printf(TEXT("%s \"%s\""), *SourceFilesList, *SourcePath);
	}

	// Trim any whitespace on our source file list
	SourceFilesList.TrimStartInline();
	SourceFilesList.TrimEndInline();

	FProcHandle Proc = FPlatformProcess::CreateProc(*ExecutablePath, *SourceFilesList, true, false, false, nullptr, 0, nullptr, nullptr);
	if (!Proc.IsValid())
	{
		UE_LOGF(LogCLionAccessor, Warning, "Opening the source file (%ls) failed.", *SourceFilesList);
		FPlatformProcess::CloseProc(Proc);
		return false;
	}

	return true;
}

bool FCLionSourceCodeAccessor::SaveAllOpenDocuments() const
{
	//@todo.clion This feature will be made available in 2017.3, till then we'll leave it commented out for a future PR
	// FProcHandle Proc = FPlatformProcess::CreateProc(*ExecutablePath, TEXT("save"), true, false,
	//                                                 false, nullptr, 0, nullptr, nullptr);

	// if (!Proc.IsValid())
	// {
	// 	FPlatformProcess::CloseProc(Proc);
	// 	return false;
	// }
	// return true;
	return false;
}

#if PLATFORM_LINUX

// Returns true if a valid Exec path was found therefore OutExecPath is correctly set
static bool GetExecPathFromDesktopFile(const FString& DesktopFilePath, FString& OutExecPath)
{
	if (!FPaths::FileExists(DesktopFilePath))
	{
		return false;
	}
	
	FString DesktopFileContents;
	FFileHelper::LoadFileToString(DesktopFileContents, *DesktopFilePath);

	const bool Result = FParse::Value(*DesktopFileContents, TEXT("Exec="), OutExecPath);
	if (!FPaths::FileExists(OutExecPath))
	{
		return false;
	}
	
	return Result;
}

#endif

FString FCLionSourceCodeAccessor::FindExecutablePath()
{
#if PLATFORM_WINDOWS
	// Search from JetBrainsToolbox folder
	FString ToolboxBinPath;

	if (FWindowsPlatformMisc::QueryRegKey(HKEY_CURRENT_USER, TEXT("Software\\JetBrains\\Toolbox\\"), TEXT(""), ToolboxBinPath))
	{
		FPaths::NormalizeDirectoryName(ToolboxBinPath);
		FString PatternString(TEXT("(.*)/bin"));
		FRegexPattern Pattern(PatternString);
		FRegexMatcher Matcher(Pattern, ToolboxBinPath);
		if (Matcher.FindNext())
		{
			FString ToolboxPath = Matcher.GetCaptureGroup(1);

			FString SettingJsonPath = FPaths::Combine(ToolboxPath, FString(".settings.json"));
			if (FPaths::FileExists(SettingJsonPath))
			{
				FString JsonStr;
				FFileHelper::LoadFileToString(JsonStr, *SettingJsonPath);
				TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonStr);
				TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject());
				if (FJsonSerializer::Deserialize(JsonReader, JsonObject) && JsonObject.IsValid())
				{
					FString InstallLocation;
					if (JsonObject->TryGetStringField(TEXT("install_location"), InstallLocation))
					{
						if (!InstallLocation.IsEmpty())
						{
							ToolboxPath = InstallLocation;
						}
					}
				}
			}

			FString CLionHome = FPaths::Combine(ToolboxPath, FString("apps"), FString("CLion"));
			if (FPaths::DirectoryExists(CLionHome))
			{
				TArray<FString> IDEPaths;
				IFileManager::Get().FindFilesRecursive(IDEPaths, *CLionHome, TEXT("clion64.exe"), true, false);
				if (IDEPaths.Num() > 0)
				{
					return IDEPaths[0];
				}
			}
		}
	}
	
	// Search from ProgID
	FString OpenCommand;
	if (!FWindowsPlatformMisc::QueryRegKey(HKEY_CURRENT_USER, TEXT("SOFTWARE\\Classes\\Applications\\clion64.exe\\shell\\open\\command\\"), TEXT(""), OpenCommand))
	{
		FWindowsPlatformMisc::QueryRegKey(HKEY_LOCAL_MACHINE, TEXT("SOFTWARE\\Classes\\Applications\\clion64.exe\\shell\\open\\command\\"), TEXT(""), OpenCommand);
	}

	FString PatternString(TEXT("\"(.*)\" \".*\""));
	FRegexPattern Pattern(PatternString);
	FRegexMatcher Matcher(Pattern, OpenCommand);
	if (Matcher.FindNext())
	{
		FString IDEPath = Matcher.GetCaptureGroup(1);
		if (FPaths::FileExists(IDEPath))
		{
			return IDEPath;
		}
	}

#elif  PLATFORM_MAC

	// Check for EAP
	NSURL* CLionPreviewURL = [[NSWorkspace sharedWorkspace] URLForApplicationWithBundleIdentifier:@"com.jetbrains.CLion-EAP"];
	if (CLionPreviewURL != nullptr)
	{
		return FString([CLionPreviewURL path]);
	}

	// Standard CLion Install
	NSURL* CLionURL = [[NSWorkspace sharedWorkspace] URLForApplicationWithBundleIdentifier:@"com.jetbrains.CLion"];
	if (CLionURL != nullptr)
	{
		return FString([CLionURL path]);
	}

	// Failsafe
	if (FPaths::FileExists(TEXT("/Applications/CLion.app/Contents/MacOS/clion")))
	{
		return TEXT("/Applications/CLion.app/Contents/MacOS/clion");
	}

#else
	
	// Potential executable location
	if (FPaths::FileExists(TEXT("/opt/clion/bin/clion.sh")))
	{
		return TEXT("/opt/clion/bin/clion.sh");
	}
	
	FString ClionPath;
	
	// First potential .desktop file location
	FString DesktopFilePath = TEXT("/usr/share/applications/jetbrains-clion.desktop");
	if (GetExecPathFromDesktopFile(DesktopFilePath, ClionPath))
	{
		return ClionPath;
	}
	
	// Second potential .desktop file location
	DesktopFilePath = FPaths::Combine(FPlatformMisc::GetEnvironmentVariable(TEXT("HOME")), TEXT("/.local/share/applications/jetbrains-clion.desktop"));
	if (GetExecPathFromDesktopFile(DesktopFilePath, ClionPath))
	{
		return ClionPath;
	}
	
	// Check to see if CLion is in the $PATH
	TArray<FString> PathArray;
	FPlatformMisc::GetEnvironmentVariable(TEXT("PATH")).ParseIntoArray(PathArray, FPlatformMisc::GetPathVarDelimiter());
	
	for (const FString& Path : PathArray)
	{
		if (Path.Contains("CLion"))
		{
			ClionPath = FPaths::Combine(Path, TEXT("clion.sh"));
			if (FPaths::FileExists(ClionPath))
			{
				return ClionPath;
			}
		}
	}

#endif

	// Nothing was found, return nothing as well
	return TEXT("");
}

#undef LOCTEXT_NAMESPACE

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif
