// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceCodeAccessor.h"

class FCLionSourceCodeAccessor : public ISourceCodeAccessor
{
public:
	/** ISourceCodeAccessor implementation */
	virtual void RefreshAvailability() override;

	virtual bool CanAccessSourceCode() const override;

	virtual bool DoesSolutionExist() const override;

	virtual FName GetFName() const override;

	virtual FText GetNameText() const override;

	virtual FText GetDescriptionText() const override;

	virtual bool OpenSolution() override;

	virtual bool OpenSolutionAtPath(const FString& InSolutionPath) override;

	virtual bool OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0) override;

	virtual bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) override;

	virtual bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules) override;

	virtual bool SaveAllOpenDocuments() const override;

	/**
	 * Frame Tick (Not Used)
	 * @param DeltaTime of frame
	 */
	virtual void Tick(const float DeltaTime) override
	{
	};


private:

	/**
	 * Is CLion installed on this system?
	 */
	bool bHasCLionInstalled = false;
	/**
	 * The path to the CLion executable.
	 */
	FString ExecutablePath;

	/**
	 * Search for CLion installation
	 * @return Executable path
	 */
	FString FindExecutablePath();
};
